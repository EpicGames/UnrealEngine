// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LinuxArm64TargetPlatform : ModuleRules
{
    public LinuxArm64TargetPlatform(ReadOnlyTargetRules Target) : base(Target)
    {
        BinariesSubFolder = "LinuxArm64";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.LinuxArm64);

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"DesktopPlatform",
				"TargetPlatform",
			}
        );
    }
}
