// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinuxArm64NoEditorTargetPlatformModule.cpp: Implements the FLinuxArm64NoEditorTargetPlatformModule class.
=============================================================================*/

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Common/TargetPlatformBase.h"

/**
 * Module for the Linux target platforms
 */
class FLinuxArm64TargetPlatformModule
	: public ITargetPlatformModule
{
public:

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms) override
	{
	}

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls) override
	{
		for (ITargetPlatformControls* TargetPlatformControlsIt : TargetPlatformControls)
		{
			TargetPlatforms.Add(new FTargetPlatformMerged(TargetPlatformControlsIt->GetTargetPlatformSettings(), TargetPlatformControlsIt));
		}
	}
};


IMPLEMENT_MODULE(FLinuxArm64TargetPlatformModule, LinuxArm64TargetPlatform);
