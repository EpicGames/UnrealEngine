// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LinuxArm64TargetPlatformControls : ModuleRules
{
    public LinuxArm64TargetPlatformControls(ReadOnlyTargetRules Target) : base(Target)
    {
        BinariesSubFolder = "LinuxArm64";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.LinuxArm64);

		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "LArm64TPCon";

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"DesktopPlatform",
				"TargetPlatform",
				"LinuxArm64TargetPlatformSettings",
			}
        );

        if (Target.bCompileAgainstEngine)
        {
            PrivateDependencyModuleNames.AddRange(new string[] {
					"Engine"
				}
            );

            PrivateIncludePathModuleNames.Add("TextureCompressor");
        }

		PrivateIncludePathModuleNames.Add("LinuxArm64TargetPlatformSettings");
		PrivateIncludePathModuleNames.Add("LinuxTargetPlatformSettings");
		PrivateIncludePathModuleNames.Add("LinuxTargetPlatformControls");
    }
}
