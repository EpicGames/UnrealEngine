// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FLinuxArm64TargetPlatformControlsModule.cpp: Implements the FLinuxArm64TargetPlatformControlsModule class.
=============================================================================*/

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "ILinuxArm64TargetPlatformControlsModule.h"
#include "LinuxTargetPlatformSettings.h"
#include "LinuxTargetPlatformControls.h"
#include "ILinuxArm64TargetPlatformSettingsModule.h"

/**
 * Module for the Linux ARM64 target platform controls
 */
class FLinuxArm64TargetPlatformControlsModule
	: public ILinuxArm64TargetPlatformControlsModule
{
public:
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) override
	{
		ILinuxArm64TargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<ILinuxArm64TargetPlatformSettingsModule>(PlatformSettingsModuleName);
		if (ModuleSettings != nullptr)
		{
			// ensure this is only called once (matches Android's convention of guarding inside the settings-available branch)
			check(SinglePlatforms.IsEmpty());
			const FString GameTPName   = FLinuxPlatformProperties<false, false, false, true>::PlatformName();
			const FString ServerTPName = FLinuxPlatformProperties<false, true,  false, true>::PlatformName();
			const FString ClientTPName = FLinuxPlatformProperties<false, false, true,  true>::PlatformName();

			TMap<FString, ITargetPlatformSettings*> OutMap;
			ModuleSettings->GetPlatformSettingsMaps(OutMap);

			// --- Flavorless (default) ---
			TargetPlatforms.Add(new TLinuxTargetPlatformControls<FLinuxPlatformProperties<false, false, false, true>>(OutMap[GameTPName]));
			TargetPlatforms.Add(new TLinuxTargetPlatformControls<FLinuxPlatformProperties<false, true,  false, true>>(OutMap[ServerTPName]));
			TargetPlatforms.Add(new TLinuxTargetPlatformControls<FLinuxPlatformProperties<false, false, true,  true>>(OutMap[ClientTPName]));

			// --- DXT ---
			SinglePlatforms.Add(new TLinuxDXTTargetPlatformControls<FLinuxPlatformProperties<false, false, false, true>>(OutMap[GameTPName   + TEXT("_DXT")]));
			SinglePlatforms.Add(new TLinuxDXTTargetPlatformControls<FLinuxPlatformProperties<false, true,  false, true>>(OutMap[ServerTPName + TEXT("_DXT")]));
			SinglePlatforms.Add(new TLinuxDXTTargetPlatformControls<FLinuxPlatformProperties<false, false, true,  true>>(OutMap[ClientTPName + TEXT("_DXT")]));

			// --- ASTC ---
			SinglePlatforms.Add(new TLinuxASTCTargetPlatformControls<FLinuxPlatformProperties<false, false, false, true>>(OutMap[GameTPName   + TEXT("_ASTC")]));
			SinglePlatforms.Add(new TLinuxASTCTargetPlatformControls<FLinuxPlatformProperties<false, true,  false, true>>(OutMap[ServerTPName + TEXT("_ASTC")]));
			SinglePlatforms.Add(new TLinuxASTCTargetPlatformControls<FLinuxPlatformProperties<false, false, true,  true>>(OutMap[ClientTPName + TEXT("_ASTC")]));

			// --- ETC2 ---
			SinglePlatforms.Add(new TLinuxETC2TargetPlatformControls<FLinuxPlatformProperties<false, false, false, true>>(OutMap[GameTPName   + TEXT("_ETC2")]));
			SinglePlatforms.Add(new TLinuxETC2TargetPlatformControls<FLinuxPlatformProperties<false, true,  false, true>>(OutMap[ServerTPName + TEXT("_ETC2")]));
			SinglePlatforms.Add(new TLinuxETC2TargetPlatformControls<FLinuxPlatformProperties<false, false, true,  true>>(OutMap[ClientTPName + TEXT("_ETC2")]));

			TargetPlatforms.Append(SinglePlatforms);

			// --- Multi ---
			MultiGameTP   = new TLinuxMultiTargetPlatformControls<FLinuxPlatformProperties<false, false, false, true>>(OutMap[GameTPName   + TEXT("_Multi")]);
			MultiServerTP = new TLinuxMultiTargetPlatformControls<FLinuxPlatformProperties<false, true,  false, true>>(OutMap[ServerTPName + TEXT("_Multi")]);
			MultiClientTP = new TLinuxMultiTargetPlatformControls<FLinuxPlatformProperties<false, false, true,  true>>(OutMap[ClientTPName + TEXT("_Multi")]);
			TargetPlatforms.Add(MultiGameTP);
			TargetPlatforms.Add(MultiServerTP);
			TargetPlatforms.Add(MultiClientTP);

			// Load the multi format selection from config now that all single-format platforms are ready
			NotifyMultiSelectedFormatsChanged();
		}
	}

	virtual void NotifyMultiSelectedFormatsChanged() override
	{
		if (MultiGameTP && MultiServerTP && MultiClientTP)
		{
			MultiGameTP->LoadFormats(SinglePlatforms);
			MultiServerTP->LoadFormats(SinglePlatforms);
			MultiClientTP->LoadFormats(SinglePlatforms);
		}
	}

private:
	// Allocated platform controls live for the lifetime of this module. They are intentionally not deleted on unload,
	// consistent with Android, iOS, Windows, Mac, and Linux modules which follow the same convention.
	// GetTargetPlatformControls is guarded against multiple calls (see check above), so the objects pointed to by
	// SinglePlatforms remain valid for the module's lifetime. LoadFormats() takes the array by value (a shallow copy
	// of 9 pointers), which is safe because the pointed-to objects outlive the call.

	/** Single-format cook targets (DXT, ASTC, ETC2), used to populate Multi on demand */
	TArray<ITargetPlatformControls*> SinglePlatforms;

	TLinuxMultiTargetPlatformControls<FLinuxPlatformProperties<false, false, false, true>>* MultiGameTP   = nullptr;
	TLinuxMultiTargetPlatformControls<FLinuxPlatformProperties<false, true,  false, true>>* MultiServerTP = nullptr;
	TLinuxMultiTargetPlatformControls<FLinuxPlatformProperties<false, false, true,  true>>* MultiClientTP = nullptr;
};

IMPLEMENT_MODULE(FLinuxArm64TargetPlatformControlsModule, LinuxArm64TargetPlatformControls);
