// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformControlsModule.h"

class ILinuxArm64TargetPlatformControlsModule : public ITargetPlatformControlsModule
{
public:
	// Called when the user changes the selected texture formats for the Multi cook target
	virtual void NotifyMultiSelectedFormatsChanged() = 0;
};
