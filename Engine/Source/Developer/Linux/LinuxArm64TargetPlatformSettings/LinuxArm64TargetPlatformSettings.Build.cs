// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LinuxArm64TargetPlatformSettings : ModuleRules
{
    public LinuxArm64TargetPlatformSettings(ReadOnlyTargetRules Target) : base(Target)
    {
        BinariesSubFolder = "LinuxArm64";
		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "LArm64TPSet";

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"DesktopPlatform",
				"TargetPlatform",
			}
        );

        if (Target.bCompileAgainstEngine)
        {
            PrivateDependencyModuleNames.AddRange(new string[] {
					"Engine"
				}
            );

            PrivateIncludePathModuleNames.Add("TextureCompressor");
        }

		PrivateIncludePathModuleNames.Add("LinuxTargetPlatformSettings");
	}
}
