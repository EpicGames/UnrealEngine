// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinuxArm64TargetPlatformSettingsModule.cpp: Implements the FLinuxArm64TargetPlatformSettingsModule class.
=============================================================================*/

#include "Modules/ModuleManager.h"
#include "ILinuxArm64TargetPlatformSettingsModule.h"
#include "LinuxTargetPlatformSettings.h"


/**
 * Module for the Linux target platforms settings
 */
class FLinuxArm64TargetPlatformSettingsModule
	: public ILinuxArm64TargetPlatformSettingsModule
{
public:

	// This follows the same single-call convention as all other platform settings modules (Android, iOS, Windows, Mac, Linux).
	// The caller is responsible for ensuring it is only called once; no multiple-call guard is needed here.
	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms) override
	{
		const FString NoEditorTPName = FLinuxPlatformProperties<false, false, false, true>::PlatformName();
		const FString ServerTPName = FLinuxPlatformProperties<false, true, false, true>::PlatformName();
		const FString ClientTPName = FLinuxPlatformProperties<false, false, true, true>::PlatformName();
	
		// NoEditor TP
		ITargetPlatformSettings* NoEditorTP = new TLinuxTargetPlatformSettings<FLinuxPlatformProperties<false, false, false, true>>();
		TargetPlatforms.Add(NoEditorTP);
		PlatformNameToPlatformSettings.Add(NoEditorTPName, NoEditorTP);
		// Server TP
		ITargetPlatformSettings* ServerTP = new TLinuxTargetPlatformSettings<FLinuxPlatformProperties<false, true, false, true>>();
		TargetPlatforms.Add(ServerTP);
		PlatformNameToPlatformSettings.Add(ServerTPName, ServerTP);
		// Client TP
		ITargetPlatformSettings* ClientTP = new TLinuxTargetPlatformSettings<FLinuxPlatformProperties<false, false, true, true>>();
		TargetPlatforms.Add(ClientTP);
		PlatformNameToPlatformSettings.Add(ClientTPName, ClientTP);

		// --- DXT Variants ---
		// NoEditor TP
		NoEditorTP = new TLinux_DXTTargetPlatformSettings<FLinuxPlatformProperties<false, false, false, true>>();
		TargetPlatforms.Add(NoEditorTP);
		PlatformNameToPlatformSettings.Add(NoEditorTPName + TEXT("_DXT"), NoEditorTP);
		// Server TP
		ServerTP = new TLinux_DXTTargetPlatformSettings<FLinuxPlatformProperties<false, true, false, true>>();
		TargetPlatforms.Add(ServerTP);
		PlatformNameToPlatformSettings.Add(ServerTPName + TEXT("_DXT"), ServerTP);
		// Client TP
		ClientTP = new TLinux_DXTTargetPlatformSettings<FLinuxPlatformProperties<false, false, true, true>>();
		TargetPlatforms.Add(ClientTP);
		PlatformNameToPlatformSettings.Add(ClientTPName + TEXT("_DXT"), ClientTP);

		// --- ASTC Variants ---
		// NoEditor TP
		NoEditorTP = new TLinux_ASTCTargetPlatformSettings<FLinuxPlatformProperties<false, false, false, true>>();
		TargetPlatforms.Add(NoEditorTP);
		PlatformNameToPlatformSettings.Add(NoEditorTPName + TEXT("_ASTC"), NoEditorTP);
		// Server TP
 		ServerTP = new TLinux_ASTCTargetPlatformSettings<FLinuxPlatformProperties<false, true, false, true>>();
 		TargetPlatforms.Add(ServerTP);
 		PlatformNameToPlatformSettings.Add(ServerTPName + TEXT("_ASTC"), ServerTP);
 		// Client TP
 		ClientTP = new TLinux_ASTCTargetPlatformSettings<FLinuxPlatformProperties<false, false, true, true>>();
 		TargetPlatforms.Add(ClientTP);
 		PlatformNameToPlatformSettings.Add(ClientTPName + TEXT("_ASTC"), ClientTP);
 		 
 		// --- ETC2 Variants ---
 		NoEditorTP = new TLinux_ETC2TargetPlatformSettings<FLinuxPlatformProperties<false, false, false, true>>();
 		TargetPlatforms.Add(NoEditorTP);
 		PlatformNameToPlatformSettings.Add(NoEditorTPName + TEXT("_ETC2"), NoEditorTP);
 		// Server TP
 		ServerTP = new TLinux_ETC2TargetPlatformSettings<FLinuxPlatformProperties<false, true, false, true>>();
 		TargetPlatforms.Add(ServerTP);
 		PlatformNameToPlatformSettings.Add(ServerTPName + TEXT("_ETC2"), ServerTP);
 		// Client TP
 		ClientTP = new TLinux_ETC2TargetPlatformSettings<FLinuxPlatformProperties<false, false, true, true>>();
 		TargetPlatforms.Add(ClientTP);
 		PlatformNameToPlatformSettings.Add(ClientTPName + TEXT("_ETC2"), ClientTP);

		// --- Multi Variants ---
		// NoEditor TP
		NoEditorTP = new TLinux_MultiTargetPlatformSettings<FLinuxPlatformProperties<false, false, false, true>>();
		TargetPlatforms.Add(NoEditorTP);
		PlatformNameToPlatformSettings.Add(NoEditorTPName + TEXT("_Multi"), NoEditorTP);
		// Server TP
		ServerTP = new TLinux_MultiTargetPlatformSettings<FLinuxPlatformProperties<false, true, false, true>>();
		TargetPlatforms.Add(ServerTP);
		PlatformNameToPlatformSettings.Add(ServerTPName + TEXT("_Multi"), ServerTP);
		// Client TP
		ClientTP = new TLinux_MultiTargetPlatformSettings<FLinuxPlatformProperties<false, false, true, true>>();
		TargetPlatforms.Add(ClientTP);
		PlatformNameToPlatformSettings.Add(ClientTPName + TEXT("_Multi"), ClientTP);
	}

	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) override
	{
		OutMap = PlatformNameToPlatformSettings;
	}

private:
	TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;

};

IMPLEMENT_MODULE(FLinuxArm64TargetPlatformSettingsModule, LinuxArm64TargetPlatformSettings);
