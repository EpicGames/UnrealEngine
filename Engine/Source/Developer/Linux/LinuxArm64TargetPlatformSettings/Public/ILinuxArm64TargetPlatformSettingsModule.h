// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once 

/*------------------------------------------------------------------------------------
IWindowsTargetPlatformSettingsModule interface
------------------------------------------------------------------------------------*/

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"

class ILinuxArm64TargetPlatformSettingsModule : public ITargetPlatformSettingsModule
{
public:
	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) = 0;
};
