// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LinuxPlatformEditor : ModuleRules
{
	public LinuxPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"DesktopPlatform",
				"Engine",
				"MainFrame",
				"Slate",
				"SlateCore",
				
				"PropertyEditor",
				"SharedSettingsWidgets",
				"SourceControl",
				"LinuxTargetPlatformSettings",
				"LinuxTargetPlatformControls",
				"TargetPlatform",
				"MaterialShaderQualitySettings",
				"RenderCore",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
				"Settings",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
				}
		);
	}
}
