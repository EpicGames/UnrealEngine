// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "LinuxTargetSettings.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "LinuxPlatformEditorModule"


/**
 * Module for Linux project settings
 */
class FLinuxPlatformEditorModule
	: public IModuleInterface
{
	// IModuleInterface interface

	virtual void StartupModule() override
	{
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "Linux",
				LOCTEXT("TargetSettingsName", "Linux"),
				LOCTEXT("TargetSettingsDescription", "Settings for Linux target platform"),
				GetMutableDefault<ULinuxTargetSettings>()
			);
		}
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "Linux");
		}
	}
};


IMPLEMENT_MODULE(FLinuxPlatformEditorModule, LinuxPlatformEditor);

#undef LOCTEXT_NAMESPACE
