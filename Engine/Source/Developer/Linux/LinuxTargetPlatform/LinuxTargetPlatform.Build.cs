// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LinuxTargetPlatform : ModuleRules
{
    public LinuxTargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
        BinariesSubFolder = "Linux";
		SDKVersionRelevantPlatforms.AddRange(Utils.GetPlatformsInGroup(UnrealPlatformGroup.Linux));

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"TargetPlatform",
				"DesktopPlatform",
				"LinuxTargetPlatformSettings",
				"LinuxTargetPlatformControls",
			}
        );
	}
}
