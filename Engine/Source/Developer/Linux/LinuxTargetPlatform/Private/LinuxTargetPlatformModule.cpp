// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Common/TargetPlatformBase.h"

#define LOCTEXT_NAMESPACE "FLinuxTargetPlatformModule"

/**
 * Module for the Linux target platform.
 */
class FLinuxTargetPlatformModule
	: public ITargetPlatformModule
{
public:
	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms)
	{

	}

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls) override
	{
		for (ITargetPlatformControls* TargetPlatformControlsIt : TargetPlatformControls)
		{
			TargetPlatforms.Add(new FTargetPlatformMerged(TargetPlatformControlsIt->GetTargetPlatformSettings(), TargetPlatformControlsIt));
		}
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{

	}

	virtual void ShutdownModule() override
	{

	}
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FLinuxTargetPlatformModule, LinuxTargetPlatform);
