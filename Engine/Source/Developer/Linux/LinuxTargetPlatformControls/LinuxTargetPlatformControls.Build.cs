// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LinuxTargetPlatformControls : ModuleRules
{
    public LinuxTargetPlatformControls(ReadOnlyTargetRules Target) : base(Target)
	{
        BinariesSubFolder = "Linux";
		SDKVersionRelevantPlatforms.AddRange(Utils.GetPlatformsInGroup(UnrealPlatformGroup.Linux));

		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "LinuxTPCon";

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
				"Projects",
				"LinuxTargetPlatformSettings",
			}
        );

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings"
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PublicIncludePathModuleNames.Add("Engine");
			PrivateDependencyModuleNames.Add("Engine");
			PrivateIncludePathModuleNames.Add("TextureCompressor");
		}
	}
}
