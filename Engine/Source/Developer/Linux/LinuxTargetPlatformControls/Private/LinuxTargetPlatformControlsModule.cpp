// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "LinuxTargetPlatformControls.h"
#include "ILinuxTargetPlatformSettingsModule.h"

#define LOCTEXT_NAMESPACE "FLinuxTargetPlatformControlModule"



/**
 * Module for the Linux target platform controls.
 */
class FLinuxTargetPlatformControlsModule
	: public ITargetPlatformControlsModule
{
public:

	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) override
	{
		TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;
		ILinuxTargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<ILinuxTargetPlatformSettingsModule>(PlatformSettingsModuleName);
		if (ModuleSettings != nullptr)
		{
			TMap<FString, ITargetPlatformSettings*> OutMap;
			ModuleSettings->GetPlatformSettingsMaps(OutMap);
			ITargetPlatformControls* GameTP = new TLinuxTargetPlatformControls<FLinuxPlatformProperties<false, false, false, false>>(OutMap[FLinuxPlatformProperties<false, false, false, false>::PlatformName()]);
			ITargetPlatformControls* EditorTP = new TLinuxTargetPlatformControls<FLinuxPlatformProperties<true, false, false, false>>(OutMap[FLinuxPlatformProperties<true, false, false, false>::PlatformName()]);
			ITargetPlatformControls* ServerTP = new TLinuxTargetPlatformControls<FLinuxPlatformProperties<false, true, false, false>>(OutMap[FLinuxPlatformProperties<false, true, false, false>::PlatformName()]);
			ITargetPlatformControls* ClientTP = new TLinuxTargetPlatformControls<FLinuxPlatformProperties<false, false, true, false>>(OutMap[FLinuxPlatformProperties<false, false, true, false>::PlatformName()]);
			TargetPlatforms.Add(GameTP);
			TargetPlatforms.Add(EditorTP);
			TargetPlatforms.Add(ServerTP);
			TargetPlatforms.Add(ClientTP);
		}
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{

	}
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FLinuxTargetPlatformControlsModule, LinuxTargetPlatformControls);
