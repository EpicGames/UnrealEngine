// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetDevice.h"
#include "Interfaces/TargetDeviceId.h"

#if PLATFORM_LINUX
	#include <signal.h>
	#include <pwd.h>
#endif // PLATFORM_LINUX

class IFileManager;
struct FProcHandle;


/**
 * Type definition for shared pointers to instances of FLinuxTargetDevice.
 */
typedef TSharedPtr<class FLinuxTargetDevice, ESPMode::ThreadSafe> FLinuxTargetDevicePtr;

/**
 * Type definition for shared references to instances of FLinuxTargetDevice.
 */
typedef TSharedRef<class FLinuxTargetDevice, ESPMode::ThreadSafe> FLinuxTargetDeviceRef;


/**
 * Implements a Linux target device.
 */
class FLinuxTargetDevice
	: public ITargetDevice
{
public:

	/**
	 * Creates and initializes a new device for the specified target platform.
	 *
	 * @param InTargetPlatformControls - The target platform controls.
	 */
	FLinuxTargetDevice(const ITargetPlatformControls& InTargetPlatformControls, const FString& InDeviceName, TFunction<void()> InSavePlatformDevices)
		: TargetPlatformControls(InTargetPlatformControls)
		, DeviceName(InDeviceName)
		, SavePlatformDevices(InSavePlatformDevices)
	{ }

	/**
	 * Minimal constructor for use with SteamDeck devices */
	FLinuxTargetDevice(const ITargetPlatformControls& InTargetPlatformControls)
		: FLinuxTargetDevice(InTargetPlatformControls, TEXT("UnknownName"), nullptr)
	{ }

public:

	virtual bool Connect( ) override
	{
		return true;
	}

	virtual void Disconnect( ) override
	{ }

	virtual ETargetDeviceTypes GetDeviceType( ) const override
	{
		return ETargetDeviceTypes::Desktop;
	}

	virtual FTargetDeviceId GetId() const override
	{
		return FTargetDeviceId(TargetPlatformControls.PlatformName(), GetName());
	}

	virtual FString GetName( ) const override
	{
		return DeviceName;
	}

	virtual FString GetOperatingSystemName( ) override
	{
		return TEXT("GNU/Linux");
	}

	virtual int32 GetProcessSnapshot( TArray<FTargetDeviceProcessInfo>& OutProcessInfos ) override
	{
		STUBBED("FLinuxTargetDevice::GetProcessSnapshot");
		return 0;
	}

	virtual const class ITargetPlatformSettings& GetPlatformSettings() const override
	{
		return *(TargetPlatformControls.GetTargetPlatformSettings());
	}
	virtual const class ITargetPlatformControls& GetPlatformControls() const override
	{
		return TargetPlatformControls;
	}

	virtual bool IsConnected( ) override
	{
		return true;
	}

	virtual bool IsDefault( ) const override
	{
		return true;
	}

	virtual bool PowerOff( bool Force ) override
	{
		return false;
	}

	virtual bool PowerOn( ) override
	{
		return false;
	}

	virtual bool Reboot( bool bReconnect = false ) override
	{
		STUBBED("FLinuxTargetDevice::Reboot");
		return false;
	}

	virtual bool SupportsFeature( ETargetDeviceFeatures Feature ) const override
	{
		switch (Feature)
		{
		case ETargetDeviceFeatures::MultiLaunch:
			return true;

			// @todo to be implemented
		case ETargetDeviceFeatures::PowerOff:
			return false;

			// @todo to be implemented turning on remote PCs (wake on LAN)
		case ETargetDeviceFeatures::PowerOn:
			return false;

			// @todo to be implemented
		case ETargetDeviceFeatures::ProcessSnapshot:
			return false;

			// @todo to be implemented
		case ETargetDeviceFeatures::Reboot:
			return false;
		}

		return false;
	}

	virtual void SetUserCredentials( const FString& InUserName, const FString& InUserPassword ) override
	{
		UserName = InUserName;
		UserPassword = InUserPassword;

		if (SavePlatformDevices)
		{
			SavePlatformDevices();
		}
	}

	virtual bool GetUserCredentials( FString& OutUserName, FString& OutUserPassword ) override
	{
		OutUserName = UserName;
		OutUserPassword = UserPassword;
		return true;
	}

	virtual bool TerminateProcess( const int64 ProcessId ) override
	{
#if PLATFORM_LINUX // if running natively, just terminate the local process
		// get process path from the ProcessId
		const int32 ReadLinkSize = 1024;
		char ReadLinkCmd[ReadLinkSize] = { 0 };
		FCStringAnsi::Sprintf(ReadLinkCmd, "/proc/%lld/exe", ProcessId);
		char ProcessPath[UNIX_MAX_PATH + 1] = { 0 };
		int32 Ret = readlink(ReadLinkCmd, ProcessPath, UE_ARRAY_COUNT(ProcessPath) - 1);
		if (Ret != -1)
		{
			struct stat st;
			uid_t euid;
			stat(ProcessPath, &st);
			euid = geteuid(); // get effective uid of current application, as this user is asking to kill a process

							  // now check if we own the process
			if (st.st_uid == euid)
			{
				// terminate it (will this terminate children as well because we set their pgid?)
				kill(ProcessId, SIGTERM);
				sleep(2); // sleep in case process still remains then send a more strict signal
				kill(ProcessId, SIGKILL);
				return true;
			}
		}
#else
		// @todo: support remote termination
		STUBBED("FLinuxTargetDevice::TerminateProcess");
#endif // PLATFORM_LINUX
		return false;
	}

protected:

	// Holds a reference to the device's target platform.
	const ITargetPlatformControls& TargetPlatformControls;

	/** Device display name */
	FString DeviceName;

	/** User name on the remote machine */
	FString UserName;

	/** User password on the remote machine */
	FString UserPassword;

	/** Target platform function to save device state */
	TFunction<void()> SavePlatformDevices;
};
