// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinuxTargetPlatformControls.h: Declares the TLinuxTargetPlatformControls class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/TargetDeviceId.h"
#include "Common/TargetPlatformControlsBase.h"
#include "SteamDeck/SteamDeckDevice.h"
#include "AnalyticsEventAttribute.h"
#include "InstalledPlatformInfo.h"
#include "LinuxTargetDevice.h"
#include "LinuxTargetPlatformSettings.h"
#include "Linux/LinuxPlatformProperties.h"
#if WITH_ENGINE
#include "TextureResource.h"
#endif // WITH_ENGINE
#define LOCTEXT_NAMESPACE "TLinuxTargetPlatformControls"

/**
 * Template for Linux target platforms controls
 */
template<typename TProperties>
class TLinuxTargetPlatformControls
	: public TTargetPlatformControlsBase<TProperties>
{
public:

	typedef TTargetPlatformControlsBase<TProperties> TSuper;

	/**
	 * Default constructor.
	 */
	TLinuxTargetPlatformControls(const PlatformInfo::FTargetPlatformInfo* const InPlatformInfo, ITargetPlatformSettings* TargetPlatformSettings)
		: TSuper(InPlatformInfo ? InPlatformInfo : new PlatformInfo::FTargetPlatformInfo(
			TProperties::IniPlatformName(),
			TProperties::HasEditorOnlyData() ? EBuildTargetType::Editor :
			TProperties::IsServerOnly() ? EBuildTargetType::Server :
			TProperties::IsClientOnly() ? EBuildTargetType::Client :
			EBuildTargetType::Game,
			TEXT("")), TargetPlatformSettings)
#if WITH_ENGINE
		, bChangingDeviceConfig(false)
#endif // WITH_ENGINE
	{
#if PLATFORM_LINUX
		if (!TProperties::IsArm64())
		{
			// only add local device if actually running on Linux
			LocalDevice = MakeShareable(new FLinuxTargetDevice(*this, FPlatformProcess::ComputerName(), nullptr));
		}
#endif

#if WITH_ENGINE

		InitDevicesFromConfig();

		if (!TProperties::IsArm64())
		{
			SteamDevices = TSteamDeckDevice<FLinuxTargetDevice>::DiscoverDevices(*this, TEXT("Native Linux"));
		}

#endif // WITH_ENGINE
	}

	// Forward this one on to the version that can create a new PlatformInfo for us.  This allows us to specify a PlatformInfo with a CookFlavor in derived
	// templates, but not have to specify the nullptr manually to keep the code compact
	TLinuxTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		: TLinuxTargetPlatformControls(nullptr, TargetPlatformSettings)
	{
	}

public:

	//~ Begin ITargetPlatform Interface

	virtual FString PlatformName() const override
	{
		// instead of TPlatformProperties (which won't have Client for non-desktop platforms), use the Info's name, which is programmatically made
		return this->PlatformInfo->Name.ToString();
	}

	virtual void EnableDeviceCheck(bool OnOff) override {}

	virtual FString CookingDeviceProfileName() const override
	{
		// when cooking for non-desktop platforms, always use the base platform name as the DP to cook with
		return TEXT("Linux");
	}

	virtual bool AddDevice(const FString& DeviceName, bool bDefault) override
	{
		return AddDevice(DeviceName, TEXT(""), TEXT(""), TEXT(""), bDefault);
	}

	virtual bool AddDevice(const FString& DeviceName, const FString& DeviceUserFriendlyName, const FString& Username, const FString& Password, bool bDefault) override
	{
		FLinuxTargetDevicePtr& Device = Devices.FindOrAdd(DeviceName);

		if (Device.IsValid())
		{
			// do not allow duplicates
			return false;
		}

		Device = MakeShareable(new FLinuxTargetDevice(*this, DeviceName,
#if WITH_ENGINE
			[&]() { SaveDevicesToConfig(); }));
		SaveDevicesToConfig();	// this will do the right thing even if AddDevice() was called from InitDevicesFromConfig
#else
			nullptr));
#endif // WITH_ENGINE

		if (!Username.IsEmpty() || !Password.IsEmpty())
		{
			Device->SetUserCredentials(Username, Password);
		}

		ITargetPlatformControls::OnDeviceDiscovered().Broadcast(Device.ToSharedRef());
		return true;
	}


	virtual void GetAllDevices(TArray<ITargetDevicePtr>& OutDevices) const override
	{
		// TODO: ping all the machines in a local segment and/or try to connect to port 22 of those that respond
		OutDevices.Reset();
		if (LocalDevice.IsValid())
		{
			OutDevices.Add(LocalDevice);
		}

		for (const auto& DeviceIter : Devices)
		{
			OutDevices.Add(DeviceIter.Value);
		}

		for (const ITargetDevicePtr& SteamDeck : SteamDevices)
		{
			if (SteamDeck.IsValid())
			{
				OutDevices.Add(SteamDeck);
			}
		}
	}

	virtual bool GenerateStreamingInstallManifest(const TMultiMap<FString, int32>& PakchunkMap, const TSet<int32>& PakchunkIndicesInUse) const override
	{
		return true;
	}

	virtual ITargetDevicePtr GetDefaultDevice() const override
	{
		if (LocalDevice.IsValid())
		{
			return LocalDevice;
		}

		return nullptr;
	}

	virtual ITargetDevicePtr GetDevice(const FTargetDeviceId& DeviceId) override
	{
		if (LocalDevice.IsValid() && (DeviceId == LocalDevice->GetId()))
		{
			return LocalDevice;
		}

		for (const auto& DeviceIter : Devices)
		{
			if (DeviceId == DeviceIter.Value->GetId())
			{
				return DeviceIter.Value;
			}
		}

		for (const ITargetDevicePtr& SteamDeck : SteamDevices)
		{
			if (SteamDeck.IsValid() && DeviceId == SteamDeck->GetId())
			{
				return SteamDeck;
			}
		}

		return nullptr;
	}

	virtual bool IsSdkInstalled(bool bProjectHasCode, FString& OutDocumentationPath) const override
	{
		if (!PLATFORM_LINUX)
		{
			// check for LINUX_MULTIARCH_ROOT or for legacy LINUX_ROOT when targeting Linux from Win/Mac

			// proceed with any value for MULTIARCH root, because checking exact architecture is not possible at this point
			FString ToolchainMultiarchRoot = FPlatformMisc::GetEnvironmentVariable(TEXT("LINUX_MULTIARCH_ROOT"));
			if (ToolchainMultiarchRoot.Len() > 0 && FPaths::DirectoryExists(ToolchainMultiarchRoot))
			{
				return true;
			}

			// else check for legacy LINUX_ROOT
			FString ToolchainCompiler = FPlatformMisc::GetEnvironmentVariable(TEXT("LINUX_ROOT"));
			if (PLATFORM_WINDOWS)
			{
				ToolchainCompiler += "/bin/clang++.exe";
			}
			else if (PLATFORM_MAC)
			{
				ToolchainCompiler += "/bin/clang++";
			}
			else
			{
				checkf(false, TEXT("Unable to target Linux on an unknown platform."));
				return false;
			}

			return FPaths::FileExists(ToolchainCompiler);
		}

		return true;
	}

	virtual int32 CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const override
	{
		int32 ReadyToBuild = TSuper::CheckRequirements(bProjectHasCode, Configuration, bRequiresAssetNativization, OutTutorialPath, OutDocumentationPath, CustomizedLogMessage);

		// do not support code/plugins in Installed builds if the required libs aren't bundled (on Windows/Mac)
		if (!PLATFORM_LINUX && !FInstalledPlatformInfo::Get().IsValidPlatform(TSuper::GetPlatformInfo().UBTPlatformString, EProjectType::Code))
		{
			if (bProjectHasCode)
			{
				ReadyToBuild |= ETargetPlatformReadyStatus::CodeUnsupported;
			}

			FText Reason;
			if (this->RequiresTempTarget(bProjectHasCode, Configuration, bRequiresAssetNativization, Reason))
			{
				ReadyToBuild |= ETargetPlatformReadyStatus::PluginsUnsupported;
			}
		}

		return ReadyToBuild;
	}

	virtual void GetPlatformSpecificProjectAnalytics(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray) const override
	{
		TSuper::GetPlatformSpecificProjectAnalytics(AnalyticsParamArray);

		TSuper::AppendAnalyticsEventConfigArray(AnalyticsParamArray, TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("TargetedRHIs"), GEngineIni);
	}

#if WITH_ENGINE

	virtual void GetTextureFormats(const UTexture* InTexture, TArray< TArray<FName> >& OutFormats) const override
	{
		if (this->AllowAudioVisualData())
		{
			// just use the standard texture format name for this texture
			GetDefaultTextureFormatNamePerLayer(OutFormats.AddDefaulted_GetRef(), this->GetTargetPlatformSettings(), this, InTexture, true, 4, true);
		}

		// Apply GLES compatibility remaps (e.g. G16->R16F, A1RGB555->RGB555A1)
		if (!OutFormats.IsEmpty())
		{
			TArray<FName>& LayerFormats = OutFormats.Last();
			for (FName& TextureFormatName : LayerFormats)
			{
				for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(LinuxTexFormat::GenericRemap); ++RemapIndex)
				{
					if (TextureFormatName == LinuxTexFormat::GenericRemap[RemapIndex][0])
					{
						TextureFormatName = LinuxTexFormat::GenericRemap[RemapIndex][1];
						break;
					}
				}
			}
		}
	}

#endif //WITH_ENGINE

	virtual bool SupportsVariants() const override
	{
		return true;
	}

	virtual float GetVariantPriority() const override
	{
		return TProperties::GetVariantPriority();
	}

	//~ End ITargetPlatform Interface

protected:

#if WITH_ENGINE
	/** Whether we're in process of changing device config - if yes, we will prevent recurrent calls. */
	bool bChangingDeviceConfig;

	void InitDevicesFromConfig()
	{
		if (bChangingDeviceConfig)
		{
			return;
		}
		bChangingDeviceConfig = true;

		int NumDevices = 0;
		for (;; ++NumDevices)
		{
			FString DeviceName, DeviceUser, DevicePass;

			FString DeviceBaseKey(FString::Printf(TEXT("LinuxTargetPlatfrom_%s_Device_%d"), *PlatformName(), NumDevices));
			FString DeviceNameKey = DeviceBaseKey + TEXT("_Name");
			if (!GConfig->GetString(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), *DeviceNameKey, DeviceName, GEngineIni))
			{
				// no such device
				break;
			}

			if (!AddDevice(DeviceName, false))
			{
				break;
			}

			// set credentials, if any
			FString DeviceUserKey = DeviceBaseKey + TEXT("_User");
			if (GConfig->GetString(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), *DeviceUserKey, DeviceUser, GEngineIni))
			{
				FString DevicePassKey = DeviceBaseKey + TEXT("_Pass");
				if (GConfig->GetString(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), *DevicePassKey, DevicePass, GEngineIni))
				{
					for (const auto& DeviceIter : Devices)
					{
						ITargetDevicePtr Device = DeviceIter.Value;
						if (Device.IsValid() && Device->GetId().GetDeviceName() == DeviceName)
						{
							Device->SetUserCredentials(DeviceUser, DevicePass);
						}
					}
				}
			}
		}

		bChangingDeviceConfig = false;
	}

	void SaveDevicesToConfig()
	{
		if (bChangingDeviceConfig)
		{
			return;
		}
		bChangingDeviceConfig = true;

		int DeviceIndex = 0;
		for (const auto& DeviceIter : Devices)
		{
			ITargetDevicePtr Device = DeviceIter.Value;

			FString DeviceBaseKey(FString::Printf(TEXT("LinuxTargetPlatfrom_%s_Device_%d"), *TSuper::PlatformName(), DeviceIndex));
			FString DeviceNameKey = DeviceBaseKey + TEXT("_Name");

			if (Device.IsValid())
			{
				FString DeviceName = Device->GetId().GetDeviceName();
				// do not save a local device on Linux or it will be duplicated
				if (PLATFORM_LINUX && DeviceName == FPlatformProcess::ComputerName())
				{
					continue;
				}

				GConfig->SetString(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), *DeviceNameKey, *DeviceName, GEngineIni);

				FString DeviceUser, DevicePass;
				if (Device->GetUserCredentials(DeviceUser, DevicePass))
				{
					FString DeviceUserKey = DeviceBaseKey + TEXT("_User");
					FString DevicePassKey = DeviceBaseKey + TEXT("_Pass");

					GConfig->SetString(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), *DeviceUserKey, *DeviceUser, GEngineIni);
					GConfig->SetString(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), *DevicePassKey, *DevicePass, GEngineIni);
				}

				++DeviceIndex;	// needs to be incremented here since we cannot allow gaps
			}
		}

		bChangingDeviceConfig = false;
	}
#endif // WITH_ENGINE

	// Holds the local device.
	FLinuxTargetDevicePtr LocalDevice;
	// Holds a map of valid devices.
	TMap<FString, FLinuxTargetDevicePtr> Devices;
	TArray<ITargetDevicePtr> SteamDevices;
};

template<typename TProperties>
class TLinuxDXTTargetPlatformControls
	: public TLinuxTargetPlatformControls<TProperties>
{
public:
	TLinuxDXTTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		: TLinuxTargetPlatformControls<TProperties>(new PlatformInfo::FTargetPlatformInfo(
			TProperties::IniPlatformName(),
			TProperties::HasEditorOnlyData() ? EBuildTargetType::Editor :
			TProperties::IsServerOnly() ? EBuildTargetType::Server :
			TProperties::IsClientOnly() ? EBuildTargetType::Client :
			EBuildTargetType::Game,
			TEXT("DXT")),
			TargetPlatformSettings)
	{
	}

	virtual float GetVariantPriority() const override
	{
		float Priority;
		return (FConfigCacheIni::ForPlatform(TEXT("Linux"))->GetFloat(LINUX_SECTION_TEXT, TEXT("TextureFormatPriority_DXT"), Priority, GEngineIni) ?
			Priority : 0.6f) * 10.0f + (TProperties::IsClientOnly() ? 0.25f : 0.5f);
	}
};

template<typename TProperties>
class TLinuxASTCTargetPlatformControls
	: public TLinuxTargetPlatformControls<TProperties>
{
public:
	TLinuxASTCTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		: TLinuxTargetPlatformControls<TProperties>(new PlatformInfo::FTargetPlatformInfo(
			TProperties::IniPlatformName(),
			TProperties::HasEditorOnlyData() ? EBuildTargetType::Editor :
			TProperties::IsServerOnly() ? EBuildTargetType::Server :
			TProperties::IsClientOnly() ? EBuildTargetType::Client :
			EBuildTargetType::Game,
			TEXT("ASTC")),
			TargetPlatformSettings)
	{
	}

	virtual float GetVariantPriority() const override
	{
		float Priority;
		return (FConfigCacheIni::ForPlatform(TEXT("Linux"))->GetFloat(LINUX_SECTION_TEXT, TEXT("TextureFormatPriority_ASTC"), Priority, GEngineIni) ?
			Priority : 0.9f) * 10.0f + (TProperties::IsClientOnly() ? 0.25f : 0.5f);
	}

#if WITH_ENGINE
	virtual void GetTextureFormats(const UTexture* InTexture, TArray< TArray<FName> >& OutFormats) const override
	{
		// call base to populate formats and apply GenericRemap
		TLinuxTargetPlatformControls<TProperties>::GetTextureFormats(InTexture, OutFormats);

		// perform any remapping away from defaults
		if (!OutFormats.IsEmpty())
		{
			TArray<FName>& LayerFormats = OutFormats.Last();
			for (FName& TextureFormatName : LayerFormats)
			{
				for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(LinuxTexFormat::ASTCRemap); ++RemapIndex)
				{
					if (TextureFormatName == LinuxTexFormat::ASTCRemap[RemapIndex][0])
					{
						TextureFormatName = LinuxTexFormat::ASTCRemap[RemapIndex][1];
						break;
					}
				}
			}

			bool bSupportASTCHDR = this->GetTargetPlatformSettings()->UsesASTCHDR();

			if (!bSupportASTCHDR)
			{
				for (FName& TextureFormatName : LayerFormats)
				{
					if (TextureFormatName == LinuxTexFormat::NameASTC_RGB_HDR)
					{
						TextureFormatName = LinuxTexFormat::NameRGBA16F;
					}
				}
			}
		}
	}

#endif //WITH_ENGINE
};

template<typename TProperties>
class TLinuxETC2TargetPlatformControls
	: public TLinuxTargetPlatformControls<TProperties>
{
public:
	TLinuxETC2TargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		: TLinuxTargetPlatformControls<TProperties>(new PlatformInfo::FTargetPlatformInfo(
			TProperties::IniPlatformName(),
			TProperties::HasEditorOnlyData() ? EBuildTargetType::Editor :
			TProperties::IsServerOnly() ? EBuildTargetType::Server :
			TProperties::IsClientOnly() ? EBuildTargetType::Client :
			EBuildTargetType::Game,
			TEXT("ETC2")),
			TargetPlatformSettings)
	{
	}

	virtual float GetVariantPriority() const override
	{
		float Priority;
		return (FConfigCacheIni::ForPlatform(TEXT("Linux"))->GetFloat(LINUX_SECTION_TEXT, TEXT("TextureFormatPriority_ETC2"), Priority, GEngineIni) ?
			Priority : 0.2f) * 10.0f + (TProperties::IsClientOnly() ? 0.25f : 0.5f);
	}

#if WITH_ENGINE
	virtual void GetTextureFormats(const UTexture* InTexture, TArray< TArray<FName> >& OutFormats) const override
	{
		// call base to populate formats and apply GenericRemap
		TLinuxTargetPlatformControls<TProperties>::GetTextureFormats(InTexture, OutFormats);

		// perform any remapping away from defaults
		if (!OutFormats.IsEmpty())
		{
			TArray<FName>& LayerFormats = OutFormats.Last();
			for (FName& TextureFormatName : LayerFormats)
			{
				for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(LinuxTexFormat::ETCRemap); ++RemapIndex)
				{
					if (TextureFormatName == LinuxTexFormat::ETCRemap[RemapIndex][0])
					{
						TextureFormatName = LinuxTexFormat::ETCRemap[RemapIndex][1];
						break;
					}
				}
			}
		}
	}

#endif // WITH_ENGINE
};

template<typename TProperties>
class TLinuxMultiTargetPlatformControls
	: public TLinuxTargetPlatformControls<TProperties>
{
	TArray<ITargetPlatformControls*> FormatTargetPlatforms;
	FString FormatTargetString;

public:
	TLinuxMultiTargetPlatformControls(ITargetPlatformSettings* InTargetPlatformSettings)
		: TLinuxTargetPlatformControls<TProperties>(new PlatformInfo::FTargetPlatformInfo(
			TProperties::IniPlatformName(),
			TProperties::HasEditorOnlyData() ? EBuildTargetType::Editor :
			TProperties::IsServerOnly() ? EBuildTargetType::Server :
			TProperties::IsClientOnly() ? EBuildTargetType::Client :
			EBuildTargetType::Game,
			TEXT("Multi")),
			InTargetPlatformSettings)
	{
	}

	// Set up which formats to include based on bMultiTargetFormat_* config keys.
	// Pass all single-format target platform controls; this will filter and deduplicate them.
	void LoadFormats(TArray<ITargetPlatformControls*> SingleFormatTPs)
	{
		// sort formats by priority so higher priority formats are packaged (and thus used by the device) first
		// use GetVariantPriority() which has hardcoded defaults when the config key is absent
		SingleFormatTPs.Sort([](const ITargetPlatformControls& A, const ITargetPlatformControls& B)
			{
				return A.GetVariantPriority() > B.GetVariantPriority();
			});

		FormatTargetPlatforms.Empty();
		FormatTargetString = TEXT("");

		// Server multi-targets don't cook visual texture data; nothing to populate.
		if (TProperties::IsServerOnly())
		{
			return;
		}

		TSet<FString> SeenFormats;

		// Only pick up single-format platforms that match our own type (Game/Client), so that
		// e.g. MultiClientTP uses Client texture settings rather than the higher-priority Game variants.
		const EBuildTargetType MyType = TProperties::IsClientOnly() ? EBuildTargetType::Client : EBuildTargetType::Game;

		for (ITargetPlatformControls* SingleFormatTP : SingleFormatTPs)
		{
			FString FlavorName = SingleFormatTP->GetTargetPlatformInfo().PlatformFlavor.ToString();
			if (FlavorName.IsEmpty())
			{
				continue;
			}

			const EBuildTargetType TPType = SingleFormatTP->GetTargetPlatformInfo().PlatformType;
			// only use each flavor once, match our platform type, and skip server platforms (servers don't cook visual data)
			if (SeenFormats.Contains(FlavorName) || TPType == EBuildTargetType::Server || TPType != MyType)
			{
				continue;
			}
			SeenFormats.Add(FlavorName);

			bool bEnabled = false;
			FString SettingsName = FString(TEXT("bMultiTargetFormat_")) + FlavorName;
			FConfigCacheIni::ForPlatform(TEXT("Linux"))->GetBool(LINUX_SECTION_TEXT, *SettingsName, bEnabled, GEngineIni);
			if (bEnabled)
			{
				if (FormatTargetPlatforms.Num())
				{
					FormatTargetString += TEXT(",");
				}
				FormatTargetString += FlavorName;
				FormatTargetPlatforms.Add(SingleFormatTP);
			}
		}

		PlatformInfo::UpdatePlatformDisplayName(
			FString(TProperties::IniPlatformName()) + TEXT("_Multi"), DisplayName());
	}

	virtual float GetVariantPriority() const override
	{
		// lowest priority so specific variants are chosen first
		return 0.5f;
	}

#if WITH_ENGINE
	virtual void GetTextureFormats(const UTexture* InTexture, TArray< TArray<FName> >& OutFormats) const override
	{
		// Ask each enabled single-format variant to choose texture formats
		for (ITargetPlatformControls* Platform : FormatTargetPlatforms)
		{
			TArray< TArray<FName> > PlatformFormats;
			Platform->GetTextureFormats(InTexture, PlatformFormats);
			for (TArray<FName>& FormatPerLayer : PlatformFormats)
			{
				OutFormats.AddUnique(FormatPerLayer);
			}
		}
	}
#endif // WITH_ENGINE

	virtual FText DisplayName() const override
	{
		return FText::Format(
			LOCTEXT("Linux_Multi", "{0} (Multi:{1})"),
			FText::FromString(TProperties::IniPlatformName()),
			FText::FromString(FormatTargetString));
	}
};

#undef LOCTEXT_NAMESPACE
