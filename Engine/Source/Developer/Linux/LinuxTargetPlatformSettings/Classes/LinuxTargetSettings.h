// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinuxTargetSettings.h: Declares the ULinuxTargetSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/TargetSettingsDefinitions.h"
#include "LinuxTargetSettings.generated.h"


/**
 * Implements the settings for the Linux target platform.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class ULinuxTargetSettings
	: public UObject
{
public:

	GENERATED_UCLASS_BODY()

	LINUXTARGETPLATFORMSETTINGS_API virtual void PostInitProperties() override;

	/** Which of the currently enabled spatialization plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SpatializationPlugin;

	/** Which of the currently enabled source data override plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SourceDataOverridePlugin;

	/** Which of the currently enabled reverb plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString ReverbPlugin;

	/** Which of the currently enabled occlusion plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString OcclusionPlugin;

	/** Quality Level to COOK SoundCues at (if set, all other levels will be stripped by the cooker). */
	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides", meta = (DisplayName = "Sound Cue Cook Quality"))
	int32 SoundCueCookQualityIndex = INDEX_NONE;

	/**
	 * The collection of RHI's we want to support on this platform.
	 * This is not always the full list of RHI we can support.
	 */
	UPROPERTY(EditAnywhere, config, Category=Rendering)
	TArray<FString> TargetedRHIs;

	/** Whether to include Nanite Fallback Meshes in cooked builds for Linux. Can be overriden for specific assets in Static Mesh Editor. */
	UPROPERTY(config, EditAnywhere, Category = "Renderer", meta = (DisplayName = "Generate Nanite Fallback Meshes", ConfigRestartRequired = true))
	bool bGenerateNaniteFallbackMeshes = true;

	/** Deprecated use RayTracingMode instead. */
	UPROPERTY(config, meta = (DeprecatedProperty))
	bool bEnableRayTracing_DEPRECATED;

	/** Controls the level of ray tracing support for Linux. r.RayTracing cvar also needs to be enabled. */
	UPROPERTY(config, EditAnywhere, Category = "Renderer", meta = (DisplayName = "Hardware Ray Tracing Mode"))
	ETargetSettingsRayTracingRuntimeMode RayTracingMode;

	// --- Linux ARM64 Multi texture format cook targets ---
	// These ARM64-specific settings are intentionally stored in ULinuxTargetSettings because both the Linux (x86-64)
	// and LinuxArm64 modules share the same ini section (/Script/LinuxTargetPlatform.LinuxTargetSettings).
	// The LinuxArm64 controls module reads these values from GEngineIni using LINUX_SECTION_TEXT.

	/** Include DXT textures when packaging with the Linux ARM64 (Multi) cook target. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "LinuxArm64 Multi Texture Formats", meta = (DisplayName = "Include DXT textures"))
	bool bMultiTargetFormat_DXT;

	/** Include ASTC textures when packaging with the Linux ARM64 (Multi) cook target. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "LinuxArm64 Multi Texture Formats", meta = (DisplayName = "Include ASTC textures"))
	bool bMultiTargetFormat_ASTC;

	/** Include ETC2 textures when packaging with the Linux ARM64 (Multi) cook target. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "LinuxArm64 Multi Texture Formats", meta = (DisplayName = "Include ETC2 textures"))
	bool bMultiTargetFormat_ETC2;

	/** Priority for the ETC2 texture format when packaging using Linux ARM64 (Multi). The highest priority format supported by the device will be used. Default value is 0.2. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "LinuxArm64 Multi Texture Formats", meta = (DisplayName = "ETC2 texture format priority"))
	float TextureFormatPriority_ETC2;

	/** Priority for the DXT texture format when packaging using Linux ARM64 (Multi). The highest priority format supported by the device will be used. Default value is 0.6. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "LinuxArm64 Multi Texture Formats", meta = (DisplayName = "DXT texture format priority"))
	float TextureFormatPriority_DXT;

	/** Priority for the ASTC texture format when packaging using Linux ARM64 (Multi). The highest priority format supported by the device will be used. Default value is 0.9. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "LinuxArm64 Multi Texture Formats", meta = (DisplayName = "ASTC texture format priority"))
	float TextureFormatPriority_ASTC;

#if WITH_EDITOR
	LINUXTARGETPLATFORMSETTINGS_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	void virtual OverrideConfigSection(FString& InOutSectionName) override
	{
		InOutSectionName = TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings");
	}
};
