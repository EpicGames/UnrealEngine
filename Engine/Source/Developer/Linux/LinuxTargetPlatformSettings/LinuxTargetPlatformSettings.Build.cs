// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LinuxTargetPlatformSettings : ModuleRules
{
    public LinuxTargetPlatformSettings(ReadOnlyTargetRules Target) : base(Target)
	{
        BinariesSubFolder = "Linux";
		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "LinuxTPSet";

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
				"Projects"
			}
        );

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings"
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PublicIncludePathModuleNames.Add("Engine");
			PrivateDependencyModuleNames.Add("Engine");
			PrivateIncludePathModuleNames.Add("TextureCompressor");
		}

		if (Target.bCompileAgainstEditor)
		{
			PrivateIncludePathModuleNames.Add("LinuxArm64TargetPlatformControls");
		}
	}
}
