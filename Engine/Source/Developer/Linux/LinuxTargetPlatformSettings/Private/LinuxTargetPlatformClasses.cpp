// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinuxTargetPlatformClasses.cpp: Implements the module's UClasses.
=============================================================================*/

#include "CoreMinimal.h"
#include "LinuxTargetSettings.h"
#include "LinuxTargetPlatformSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "ILinuxArm64TargetPlatformControlsModule.h"
#endif


/* ULinuxTargetSettings structors
 *****************************************************************************/

ULinuxTargetSettings::ULinuxTargetSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, bMultiTargetFormat_DXT(true)
	, bMultiTargetFormat_ASTC(true)
	, bMultiTargetFormat_ETC2(true)
	, TextureFormatPriority_ETC2(0.2f)
	, TextureFormatPriority_DXT(0.6f)
	, TextureFormatPriority_ASTC(0.9f)
{ }

void ULinuxTargetSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Guard against manually-edited ini files that set all Multi formats to false.
	// PostEditChangeProperty is not called on startup, so this is the only recovery path.
	if (!bMultiTargetFormat_DXT && !bMultiTargetFormat_ASTC && !bMultiTargetFormat_ETC2)
	{
		UE_LOGF(LogTemp, Warning, "LinuxTargetSettings: All bMultiTargetFormat_* flags were false (possibly from a manual ini edit); defaulting bMultiTargetFormat_ASTC to true.");
		bMultiTargetFormat_ASTC = true;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULinuxTargetSettings, bMultiTargetFormat_ASTC)), GetDefaultConfigFilename());
	}

	FProperty* RayTracingModeProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULinuxTargetSettings, RayTracingMode));
	if (RayTracingModeProperty)
	{
		FConfigFile LocalEngineIni;
		FConfigFile* EngineIni = FConfigCacheIni::FindOrLoadPlatformConfig(LocalEngineIni, TEXT("Engine"), TEXT("Linux"));

		// Convert and delete deprecated properties
		// Deprecated properties are automatically deleted from .inis when updated via UpdateSinglePropertyInConfigFile()
		bool bOldEnableRayTracing = false;
		if (EngineIni->GetBool(LINUX_SECTION_TEXT, TEXT("bEnableRayTracing"), bOldEnableRayTracing))
		{
			RayTracingMode = bOldEnableRayTracing ? ETargetSettingsRayTracingRuntimeMode::Full : ETargetSettingsRayTracingRuntimeMode::Disabled;
			UpdateSinglePropertyInConfigFile(RayTracingModeProperty, GetDefaultConfigFilename());
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(FName(TEXT("bEnableRayTracing"))), GetDefaultConfigFilename());
		}
	}
}

#if WITH_EDITOR
void ULinuxTargetSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr &&
		(PropertyChangedEvent.Property->GetName().StartsWith(TEXT("bMultiTargetFormat")) ||
		 PropertyChangedEvent.Property->GetName().StartsWith(TEXT("TextureFormatPriority"))))
	{
		// Correct in-memory state BEFORE any config writes: ensure at least one format remains selected.
		// Writing the correction first means any crash between the two config writes leaves the config valid.
		FProperty* ASTCProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULinuxTargetSettings, bMultiTargetFormat_ASTC));
		bool bWroteCorrectedASTC = false;
		if (!bMultiTargetFormat_DXT && !bMultiTargetFormat_ASTC && !bMultiTargetFormat_ETC2)
		{
			bMultiTargetFormat_ASTC = true;
			UpdateSinglePropertyInConfigFile(ASTCProperty, GetDefaultConfigFilename());
			bWroteCorrectedASTC = true;
		}

		// Skip the write if we already wrote this property as part of the correction above
		if (!bWroteCorrectedASTC || PropertyChangedEvent.Property != ASTCProperty)
		{
			UpdateSinglePropertyInConfigFile(PropertyChangedEvent.Property, GetDefaultConfigFilename());
		}

		// Notify the LinuxArm64 target platform controls module if it's loaded
		ILinuxArm64TargetPlatformControlsModule* Module = FModuleManager::GetModulePtr<ILinuxArm64TargetPlatformControlsModule>("LinuxArm64TargetPlatformControls");
		if (Module)
		{
			Module->NotifyMultiSelectedFormatsChanged();
		}
	}
}
#endif
