// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "ILinuxTargetPlatformSettingsModule.h"
#include "LinuxTargetSettings.h"
#include "LinuxTargetPlatformSettings.h"
#include "ISettingsModule.h"


#define LOCTEXT_NAMESPACE "FLinuxTargetPlatformSettingsModule"



/**
 * Module for the Linux target platform settings.
 */
class FLinuxTargetPlatformSettingsModule
	: public ILinuxTargetPlatformSettingsModule
{
public:
	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms) override
	{
		// NoEditor TP
		ITargetPlatformSettings* NoEditorTP = new TLinuxTargetPlatformSettings<FLinuxPlatformProperties<false, false, false, false>>();
		TargetPlatforms.Add(NoEditorTP);
		PlatformNameToPlatformSettings.Add(FLinuxPlatformProperties<false, false, false, false>::PlatformName(), NoEditorTP);
		// Editor TP
		ITargetPlatformSettings* EditorTP = new TLinuxTargetPlatformSettings<FLinuxPlatformProperties<true, false, false, false>>();
		TargetPlatforms.Add(EditorTP);
		PlatformNameToPlatformSettings.Add(FLinuxPlatformProperties<true, false, false, false>::PlatformName(), EditorTP);
		// Server TP
		ITargetPlatformSettings* ServerTP = new TLinuxTargetPlatformSettings<FLinuxPlatformProperties<false, true, false, false>>();
		TargetPlatforms.Add(ServerTP);
		PlatformNameToPlatformSettings.Add(FLinuxPlatformProperties<false, true, false, false>::PlatformName(), ServerTP);
		// Client TP
		ITargetPlatformSettings* ClientTP = new TLinuxTargetPlatformSettings<FLinuxPlatformProperties<false, false, true, false>>();
		TargetPlatforms.Add(ClientTP);
		PlatformNameToPlatformSettings.Add(FLinuxPlatformProperties<false, false, true, false>::PlatformName(), ClientTP);
	}

	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) override
	{
		OutMap = PlatformNameToPlatformSettings;
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		TargetSettings = NewObject<ULinuxTargetSettings>(GetTransientPackage(), "LinuxTargetSettings", RF_Standalone);

		// We need to manually load the config properties here, as this module is loaded before the UObject system is setup to do this
		GConfig->GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("TargetedRHIs"), TargetSettings->TargetedRHIs, GEngineIni);
		TargetSettings->AddToRoot();

		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "Linux",
				LOCTEXT("TargetSettingsName", "Linux"),
				LOCTEXT("TargetSettingsDescription", "Settings for Linux target platform"),
				TargetSettings
			);
		}
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "Linux");
		}

		if (!GExitPurge)
		{
			// If we're in exit purge, this object has already been destroyed
			TargetSettings->RemoveFromRoot();
		}
		else
		{
			TargetSettings = NULL;
		}
	}

private:
	TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;

	/** Holds the target settings. */
	ULinuxTargetSettings* TargetSettings;
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FLinuxTargetPlatformSettingsModule, LinuxTargetPlatformSettings);
