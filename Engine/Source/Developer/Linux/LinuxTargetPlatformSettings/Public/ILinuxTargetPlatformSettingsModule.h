// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once 

/*------------------------------------------------------------------------------------
ILinuxTargetPlatformSettingsModule interface
------------------------------------------------------------------------------------*/

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"

class ILinuxTargetPlatformSettingsModule : public ITargetPlatformSettingsModule
{
public:
	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) = 0;
};
