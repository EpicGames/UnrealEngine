// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinuxTargetPlatform.h: Declares the TLinuxTargetPlatformSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Common/TargetPlatformSettingsBase.h"
#if WITH_ENGINE
#include "StaticMeshResources.h"
#endif // WITH_ENGINE
#include "Linux/LinuxPlatformProperties.h" // IWYU pragma: export

#define LOCTEXT_NAMESPACE "TLinuxTargetPlatformSettings"
#define LINUX_SECTION_TEXT TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings")

class UTextureLODSettings;

namespace LinuxTexFormat
{
	// Compressed Texture Formats
 	const static FName NameDXT1(TEXT("DXT1"));
 	const static FName NameDXT5(TEXT("DXT5"));
 	const static FName NameDXT5n(TEXT("DXT5n"));
 	const static FName NameAutoDXT(TEXT("AutoDXT"));
 	const static FName NameBC4(TEXT("BC4"));
 	const static FName NameBC5(TEXT("BC5"));
 	const static FName NameBC6H(TEXT("BC6H"));
 	const static FName NameBC7(TEXT("BC7"));

	const static FName NameETC2_RGB(TEXT("ETC2_RGB"));
	const static FName NameETC2_RGBA(TEXT("ETC2_RGBA"));
	const static FName NameETC2_R11(TEXT("ETC2_R11"));
	const static FName NameETC2_RG11(TEXT("ETC2_RG11"));
	const static FName NameAutoETC2(TEXT("AutoETC2"));

	const static FName NameAutoASTC(TEXT("ASTC_RGBAuto"));
	const static FName NameASTC_NormalRG(TEXT("ASTC_NormalRG"));
	// L+A mode supported by ARM ASTC encoder. Not included in ASTCRemap as it is already an ASTC format and requires no remapping.
	const static FName NameASTC_NormalLA(TEXT("ASTC_NormalLA"));

	// Uncompressed Texture Formats
	const static FName NameBGRA8(TEXT("BGRA8"));
	const static FName NameG8(TEXT("G8"));
	const static FName NameRGBA16F(TEXT("RGBA16F"));
	const static FName NameR16F(TEXT("R16F"));
	const static FName NameG16(TEXT("G16"));

	//A1RGB555 is mapped to RGB555A1, because OpenGL GL_RGB5_A1 only supports alpha on the lowest bit
	const static FName NameA1RGB555(TEXT("A1RGB555"));
	const static FName NameRGB555A1(TEXT("RGB555A1"));

	const static FName GenericRemap[][2] =
	{
		{ NameA1RGB555,		NameRGB555A1		},
		{ NameG16,		NameR16F		}, // GLES does not support R16Unorm, fallback all to R16F
	};

	const static FName NameASTC_RGB_HDR(TEXT("ASTC_RGB_HDR"));
	const static FName NameASTC_RGB(TEXT("ASTC_RGB"));
	const static FName NameASTC_RGBA(TEXT("ASTC_RGBA"));
	const static FName NameASTC_RGBA_HQ(TEXT("ASTC_RGBA_HQ"));
	const static FName NameASTC_NormalAG(TEXT("ASTC_NormalAG"));

	const static FName ASTCRemap[][2] =
	{
		// Default format:		Preferred format:
		{ NameDXT1,			NameASTC_RGB			},
		{ NameDXT5,			NameASTC_RGBA			},
		{ NameDXT5n,			NameASTC_NormalAG		},
		{ NameBC5,			NameASTC_NormalRG		},
		{ NameBC4,			NameETC2_R11			},	// No ASTC single-channel format exists; ETC2_R11 is the closest equivalent
		{ NameBC6H,			NameASTC_RGB_HDR		},
		{ NameBC7,			NameASTC_RGBA_HQ		},
		{ NameAutoDXT,			NameAutoASTC			},
	};

	const static FName ETCRemap[][2] =
	{
		// Default format:	ETC2 format:
		{ NameDXT1,			NameETC2_RGB	},
		{ NameDXT5,			NameETC2_RGBA	},
		{ NameDXT5n,			NameETC2_RGB	},	// this is a quality/compatibility tradeoff that matches what Android does
		{ NameBC5,			NameETC2_RG11	},
		{ NameBC4,			NameETC2_R11	},
		{ NameBC6H,			NameRGBA16F	},
		{ NameBC7,			NameETC2_RGBA	},
		{ NameAutoDXT,			NameAutoETC2	},
	};
}

/**
 * Template for Linux target platforms settings
 */
template<typename TProperties>
class TLinuxTargetPlatformSettings
	: public TTargetPlatformSettingsBase<TProperties>
{
public:

	typedef TTargetPlatformSettingsBase<TProperties> TSuper;

	/**
	 * Default constructor.
	 */
	TLinuxTargetPlatformSettings(const TCHAR* CookFlavor = nullptr, const TCHAR* OverrideIniPlatformName = nullptr)
		: TSuper(CookFlavor, OverrideIniPlatformName)
	{
#if WITH_ENGINE
		TextureLODSettings = nullptr;
		StaticMeshLODSettings.Initialize(this);

		// Get the Target RHIs for this platform, we do not always want all those that are supported.
		TArray<FName> TargetedShaderFormats;
		TLinuxTargetPlatformSettings::GetAllTargetedShaderFormats(TargetedShaderFormats);

		// If we are targeting ES 2.0/3.1, we also must cook encoded HDR reflection captures
		static FName NAME_SF_VULKAN_ES31(TEXT("SF_VULKAN_ES31"));
		static FName NAME_OPENGL_150_ES3_1(TEXT("GLSL_150_ES31"));
		bRequiresEncodedHDRReflectionCaptures = TargetedShaderFormats.Contains(NAME_SF_VULKAN_ES31)
			|| TargetedShaderFormats.Contains(NAME_OPENGL_150_ES3_1);
#endif // WITH_ENGINE
	}


public:

	//~ Begin ITargetPlatform Interface

	virtual bool SupportsFeature(ETargetPlatformFeatures Feature) const override
	{
		if (Feature == ETargetPlatformFeatures::UserCredentials || Feature == ETargetPlatformFeatures::Packaging)
		{
			return true;
		}

		if (Feature == ETargetPlatformFeatures::CanCookPackages)
		{
			return TProperties::HasEditorOnlyData();
		}

		return TSuper::SupportsFeature(Feature);
	}

	virtual void GetAllPossibleShaderFormats( TArray<FName>& OutFormats ) const override
	{
		if (!TProperties::IsServerOnly())
		{
			static FName NAME_VULKAN_SM5(TEXT("SF_VULKAN_SM5"));
			static FName NAME_VULKAN_SM6(TEXT("SF_VULKAN_SM6"));
			static FName NAME_VULKAN_ES31(TEXT("SF_VULKAN_ES31"));

			OutFormats.AddUnique(NAME_VULKAN_SM5);
			OutFormats.AddUnique(NAME_VULKAN_SM6);
			OutFormats.AddUnique(NAME_VULKAN_ES31);
		}
	}

	virtual void GetAllTargetedShaderFormats( TArray<FName>& OutFormats ) const override
	{
		// Get the Target RHIs for this platform, we do not always want all those that are supported. (reload in case user changed in the editor)
		TArray<FString>TargetedShaderFormats;
		GConfig->GetArray(LINUX_SECTION_TEXT, TEXT("TargetedRHIs"), TargetedShaderFormats, GEngineIni);

		// Gather the list of Target RHIs and filter out any that may be invalid.
		TArray<FName> PossibleShaderFormats;
		GetAllPossibleShaderFormats(PossibleShaderFormats);

		for (int32 ShaderFormatIdx = TargetedShaderFormats.Num() - 1; ShaderFormatIdx >= 0; ShaderFormatIdx--)
		{
			FString ShaderFormat = TargetedShaderFormats[ShaderFormatIdx];
			if (PossibleShaderFormats.Contains(FName(*ShaderFormat)) == false)
			{
				TargetedShaderFormats.RemoveAt(ShaderFormatIdx);
			}
		}

		for(const FString& ShaderFormat : TargetedShaderFormats)
		{
			OutFormats.AddUnique(FName(*ShaderFormat));
		}
	}

	virtual bool IsRunningPlatform() const override
	{
		// Must be Linux platform as editor for this to be considered a running platform
		return PLATFORM_LINUX && !UE_SERVER && !UE_GAME && WITH_EDITOR && TProperties::HasEditorOnlyData();
	}

#if WITH_ENGINE
	virtual void GetReflectionCaptureFormats(TArray<FName>& OutFormats) const override
	{
		if (bRequiresEncodedHDRReflectionCaptures)
		{
			OutFormats.Add(FName(TEXT("EncodedHDR")));
		}

		OutFormats.Add(FName(TEXT("FullHDR")));
	}

	virtual const class FStaticMeshLODSettings& GetStaticMeshLODSettings( ) const override
	{
		return StaticMeshLODSettings;
	}

	virtual const UTextureLODSettings& GetTextureLODSettings() const override
	{
		return *TextureLODSettings;
	}

	virtual void RegisterTextureLODSettings(const UTextureLODSettings* InTextureLODSettings) override
	{
		TextureLODSettings = InTextureLODSettings;
	}

#endif //WITH_ENGINE

	virtual bool ShouldStripNaniteFallbackMeshes() const override
	{
		bool bGenerateNaniteFallbackMeshes = true;
		GConfig->GetBool(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("bGenerateNaniteFallbackMeshes"), bGenerateNaniteFallbackMeshes, GEngineIni);

		return !bGenerateNaniteFallbackMeshes;
	}
	
	virtual bool UsesRayTracing() const override
	{
		return GetRayTracingMode() != ERayTracingRuntimeMode::Disabled;
	}

	virtual ERayTracingRuntimeMode GetRayTracingMode() const override
	{
		if (!TTargetPlatformSettingsBase<TProperties>::UsesRayTracing())
		{
			return ERayTracingRuntimeMode::Disabled;
		}

		return TTargetPlatformSettingsBase<TProperties>::ParseRuntimeRayTracingMode(LINUX_SECTION_TEXT);
	}

	//~ End ITargetPlatform Interface

protected:


#if WITH_ENGINE
	// Holds the texture LOD settings.
	const UTextureLODSettings* TextureLODSettings;

	// Holds static mesh LOD settings.
	FStaticMeshLODSettings StaticMeshLODSettings;

	// True if the project requires encoded HDR reflection captures
	bool bRequiresEncodedHDRReflectionCaptures;
#endif // WITH_ENGINE
};

template<typename TProperties>
class TLinux_DXTTargetPlatformSettings : public TLinuxTargetPlatformSettings<TProperties>
{
public:
	TLinux_DXTTargetPlatformSettings() : TLinuxTargetPlatformSettings<TProperties>(TEXT("DXT"))
	{
	}
};

template<typename TProperties>
class TLinux_ASTCTargetPlatformSettings : public TLinuxTargetPlatformSettings<TProperties>
{
public:
	TLinux_ASTCTargetPlatformSettings() : TLinuxTargetPlatformSettings<TProperties>(TEXT("ASTC"))
	{
	}
};

template<typename TProperties>
class TLinux_ETC2TargetPlatformSettings : public TLinuxTargetPlatformSettings<TProperties>
{
public:

	TLinux_ETC2TargetPlatformSettings() : TLinuxTargetPlatformSettings<TProperties>(TEXT("ETC2"))
	{
	}
};

template<typename TProperties>
class TLinux_MultiTargetPlatformSettings : public TLinuxTargetPlatformSettings<TProperties>
{
public:

	TLinux_MultiTargetPlatformSettings() : TLinuxTargetPlatformSettings<TProperties>(TEXT("Multi"))
	{
	}
};

#undef LOCTEXT_NAMESPACE
